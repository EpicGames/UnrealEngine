{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "HttpBlueprint",
	"Description": "Allows for sending and receiving HTTP requests in Blueprint",
	"Category": "Web",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault" : false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "HttpBlueprint",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HttpBlueprintGraph",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "JsonBlueprintUtilities",
			"Enabled": true
		}
	]
}
