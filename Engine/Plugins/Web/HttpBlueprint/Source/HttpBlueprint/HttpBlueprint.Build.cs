// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HttpBlueprint : ModuleRules
{
	public HttpBlueprint(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"HTTP",
				"Slate",
				"SlateCore",
			}
		);
	}
}
