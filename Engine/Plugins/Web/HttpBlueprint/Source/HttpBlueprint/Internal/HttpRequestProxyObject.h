// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HttpHeader.h"
#include "Interfaces/IHttpRequest.h"

#include "HttpRequestProxyObject.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnRequestComplete, FString, Response, bool, bSuccessful, FHttpHeader, OutHeader);

UCLASS(MinimalAPI)
class UHttpRequestProxyObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintAssignable)
	FOnRequestComplete OnRequestComplete;

	UFUNCTION(BlueprintCallable, Meta = (BlueprintInternalUseOnly, DisplayName = "Http Request"), Category = "Http")
	static UHttpRequestProxyObject* CreateHttpRequestProxyObject(
		const FString& InUrl,
		const FString& InVerb,
		FHttpHeader InHeader,
		const FString& InBody);

protected:
	void ProcessRequest(const FString& InUrl, const FString& InVerb, FHttpHeader&& InHeader, const FString& InBody);
	void ProcessComplete(FHttpRequestPtr InRequest, FHttpResponsePtr InResponse, bool bInSuccessful);

private:
	UPROPERTY()
	FHttpHeader CachedHeader;
};
