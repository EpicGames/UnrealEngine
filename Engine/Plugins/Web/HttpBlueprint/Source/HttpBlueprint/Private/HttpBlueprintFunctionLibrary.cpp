// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpBlueprintFunctionLibrary.h"
#include "HttpHeader.h"

void UHttpBlueprintFunctionLibrary::MakeRequestHeader(const TMap<FString, FString>& Headers, FHttpHeader& OutHeader)
{
	OutHeader = FHttpHeader{}.SetHeaders(Headers);
}

bool UHttpBlueprintFunctionLibrary::GetHeaderValue(const FHttpHeader& HeaderObject, FString HeaderName, FString& OutHeaderValue)
{
	if (!HeaderObject.IsValid())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("HeaderObject can not be invalid")), ELogVerbosity::Error);
		return false;
	}

	if (HeaderName.IsEmpty())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Header Name can not be empty")), ELogVerbosity::Error);
		return false;
	}

	OutHeaderValue = HeaderObject.GetHeader(MoveTemp(HeaderName));
	return !OutHeaderValue.IsEmpty();
}

TArray<FString> UHttpBlueprintFunctionLibrary::GetAllHeaders(const FHttpHeader& HeaderObject)
{
	return HeaderObject.GetAllHeaders();
}

TMap<FString, FString> UHttpBlueprintFunctionLibrary::GetAllHeaders_Map(const FHttpHeader& HeaderObject)
{
	return HeaderObject.GetAllHeadersAsMap();
}

void UHttpBlueprintFunctionLibrary::AddHeader(FHttpHeader& HeaderObject, FString NewHeader, FString NewHeaderValue)
{
	if (NewHeader.IsEmpty())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("NewHeader can not be empty")), ELogVerbosity::Error);
		return;
	}

	if (NewHeaderValue.IsEmpty())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("NewHeaderValue can not be empty")), ELogVerbosity::Error);
		return;
	}
	
	TPair<FString, FString> NewPair(MoveTemp(NewHeader), MoveTemp(NewHeaderValue));
	HeaderObject.AddHeader(MoveTemp(NewPair));
}

bool UHttpBlueprintFunctionLibrary::RemoveHeader(FHttpHeader& HeaderObject, FString HeaderToRemove)
{
	if (HeaderToRemove.IsEmpty())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("HeaderToRemove can not be empty")), ELogVerbosity::Error);
		return false;
	}
	
	return HeaderObject.RemoveHeader(MoveTemp(HeaderToRemove));
}
