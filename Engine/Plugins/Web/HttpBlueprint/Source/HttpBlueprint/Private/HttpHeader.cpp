// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpHeader.h"

#include "HttpBlueprintTypes.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"

FHttpHeader FHttpHeader::SetHeaders(const TMap<FString, FString>& NewHeaders)
{
	Headers = NewHeaders;
	return *this;
}

FString FHttpHeader::GetHeader(FString&& Key) const
{
	const FString* FoundHeader = Headers.Find(MoveTemp(Key));
	return FoundHeader ? *FoundHeader : TEXT("");
}

void FHttpHeader::AddHeader(TPair<FString, FString>&& NewHeader)
{
	Headers.Add(MoveTemp(NewHeader));
}

bool FHttpHeader::RemoveHeader(FString&& HeaderToRemove)
{
	return !!Headers.Remove(HeaderToRemove);
}

TArray<FString> FHttpHeader::GetAllHeaders() const
{
	TArray<FString> OutArray;
	OutArray.Reserve(Headers.Num());

	for (const TPair<FString, FString>& Header : Headers)
	{
		OutArray.Add(FString::Printf(TEXT("%s: %s"), *Header.Key, *Header.Value));
	}

	return OutArray;
}

void FHttpHeader::AssignHeadersToRequest(const TSharedRef<IHttpRequest>& Request)
{
	// Add required/expected headers by default - they'll be overridden if specified below
	// Even if the values are wrong, the Http Request will notify the user of this in it's response
	const FString RequestVerb = Request->GetVerb();
	if (RequestVerb.Equals(TEXT("POST"), ESearchCase::IgnoreCase)
		|| RequestVerb.Equals(TEXT("PUT"), ESearchCase::IgnoreCase)
		|| RequestVerb.Equals(TEXT("PATCH"), ESearchCase::IgnoreCase))
	{
		const FString InferredContentType = Request->GetURL().Contains(TEXT("?"))
												? TEXT("application/x-www-form-urlencoded") // The presence of ? in a url probably means it's parameter based
												: TEXT("application/json"); // otherwise treat as having a json payload
		Request->SetHeader(TEXT("Content-Type"), InferredContentType);
	}

	for (const TPair<FString, FString>& HeaderPair : Headers)
	{
		Request->SetHeader(HeaderPair.Key, HeaderPair.Value);
	}
}
