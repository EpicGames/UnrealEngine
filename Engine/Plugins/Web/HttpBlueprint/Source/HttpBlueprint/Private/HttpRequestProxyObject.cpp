// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpRequestProxyObject.h"

#include "HttpModule.h"
#include "Interfaces/IHttpResponse.h"

UHttpRequestProxyObject* UHttpRequestProxyObject::CreateHttpRequestProxyObject(
	const FString& InUrl,
	const FString& InVerb,
	FHttpHeader InHeader,
	const FString& InBody)
{
	UHttpRequestProxyObject* const Proxy = NewObject<UHttpRequestProxyObject>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->CachedHeader = InHeader;
	Proxy->ProcessRequest(InUrl, InVerb, MoveTemp(InHeader), InBody);
	return Proxy;
}

void UHttpRequestProxyObject::ProcessRequest(
	const FString& InUrl,
	const FString& InVerb,
	FHttpHeader&& InHeader,
	const FString& InBody)
{
	FHttpModule& HttpModule = FHttpModule::Get();
	const TSharedRef<IHttpRequest> Request = HttpModule.CreateRequest();
	Request->SetURL(InUrl);
	Request->SetVerb(InVerb);
	Request->SetContentAsString(InBody);
	InHeader.AssignHeadersToRequest(Request);
	Request->ProcessRequest();

	Request->OnProcessRequestComplete().BindUObject(this, &ThisClass::ProcessComplete);
}

void UHttpRequestProxyObject::ProcessComplete(FHttpRequestPtr InRequest, FHttpResponsePtr InResponse, bool bInSuccessful)
{
	if (ensure(InRequest))
	{
		OnRequestComplete.Broadcast(InResponse ? InResponse->GetContentAsString() : LexToString(InRequest->GetFailureReason()), InResponse && bInSuccessful, MoveTemp(CachedHeader));
	}
}
