// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HttpHeader.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "HttpBlueprintFunctionLibrary.generated.h"

UCLASS()
class HTTPBLUEPRINT_API UHttpBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, BlueprintInternalUseOnly, Category = "Http")
	static void MakeRequestHeader(const TMap<FString, FString>& Headers, UPARAM(DisplayName="Header") FHttpHeader& OutHeader);

	/**
	 *	Get the value associated with a Header name
	 *
	 *	@Param HeaderObject: The structure that contains all of the headers
	 *	@Param HeaderName: The name of the Header
	 *	@Param OutHeaderValue: The value of the Header. Empty if the Header could not be found
	 *	@Return Whether the operation was successful
	 */
	UFUNCTION(BlueprintPure, Category = "Http")
	static UPARAM(DisplayName="Success") bool GetHeaderValue(const FHttpHeader& HeaderObject, FString HeaderName, FString& OutHeaderValue);

	/** Returns all of the headers and their values by value */
	UFUNCTION(BlueprintPure, Category = "Http")
	static UPARAM(DisplayName="Headers") TArray<FString> GetAllHeaders(const FHttpHeader& HeaderObject);

	/** Returns all of the headers and their values as a map by value */
	UFUNCTION(BlueprintPure, Category = "Http", meta = (DisplayName = "GetAllHeadersAsMap"))
	static UPARAM(DisplayName="Headers") TMap<FString, FString> GetAllHeaders_Map(const FHttpHeader& HeaderObject);
	
	/**
	 *	Adds a new Header
	 *
	 *	@Param HeaderObject: The structure that contains all of the headers
	 *	@Param NewHeader: The name of the header
	 *	@Param NewHeaderValue: The value of the new header
	 */
	UFUNCTION(BlueprintCallable, Category = "Http")
	static void AddHeader(UPARAM(Ref) FHttpHeader& HeaderObject, FString NewHeader, FString NewHeaderValue);

	/**
	 *	Removes a header from the HeaderObject
	 *
	 *	@Param HeaderObject: The structure that contains all of the headers
	 *	@Param HeaderToRemove: The Key of the header to remove
	 *	@Return Whether or not the operation was successful. A value of false most likely means the Key did not exist on the HeaderObject
	 */
	UFUNCTION(BlueprintCallable, Category = "Http")
	static bool RemoveHeader(UPARAM(Ref) FHttpHeader& HeaderObject, FString HeaderToRemove);
};
