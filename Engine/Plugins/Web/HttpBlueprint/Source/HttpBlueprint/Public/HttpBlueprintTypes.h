// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "HttpBlueprintTypes.generated.h"

UENUM(BlueprintType)
enum class EHttpVerbs : uint8
{
	Post	= 0,
	Put		= 1,
	Delete	= 2,
	Patch	= 3,
	// @note: Anything past Patch will not display the input body pin
	Get		= 4,

	MAX		= 255	UMETA(Hidden),
};

ENUM_CLASS_FLAGS(EHttpVerbs);

UENUM(BlueprintType)
enum class ERequestPresets : uint8
{
	// The order here matters.	
	Json			UMETA(DisplayName = "Json Request"),
	Http			UMETA(DisplayName = "Http Request"),
	Url				UMETA(DisplayName = "Url Encoded Request"),
	Custom			UMETA(DisplayName = "Custom Request")
};
