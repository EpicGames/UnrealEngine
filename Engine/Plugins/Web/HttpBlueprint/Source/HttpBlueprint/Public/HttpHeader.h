// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "HttpHeader.generated.h"

/**
 * Provides a way for blueprint to create and store a map of HTTP headers
 */
USTRUCT(BlueprintType)
struct HTTPBLUEPRINT_API FHttpHeader
{
	GENERATED_BODY()

	/**
	 * Sets the internal headers.
	 * Will overwrite any headers already in the map
	 */
	[[nodiscard]] FHttpHeader SetHeaders(const TMap<FString, FString>& NewHeaders);

	/**
	 * Returns the header value associated with the Key
	 */
	[[nodiscard]] FString GetHeader(FString&& Key) const;

	/**
	 * Adds a new header key/value pair to the internal header map
	 */
	void AddHeader(TPair<FString, FString>&& NewHeader);

	/**
	 * Removes a header from the internal header map
	 */
	[[nodiscard]] bool RemoveHeader(FString&& HeaderToRemove);

	/**
	 * Returns the internal header map as an array
	 */
	[[nodiscard]] TArray<FString> GetAllHeaders() const;

	/**
	 * Returns a copy of the internal header map
	 */
	[[nodiscard]] const TMap<FString, FString>& GetAllHeadersAsMap() const
	{
		return Headers;
	}

	/**
	 * Returns if the header is valid or not. This is done by checking the map size
	 */
	[[nodiscard]] bool IsValid() const
	{
		return Headers.Num() > 0;
	}

	/**
	 * Convenience function to set all of the headers in the internal header map on the request object.
	 */
	void AssignHeadersToRequest(const TSharedRef<class IHttpRequest>& Request);
	
private:
	/** Header key/value pairs. */
	UPROPERTY()
	TMap<FString, FString> Headers;	
};
