// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HttpBlueprintGraph : ModuleRules
{
	public HttpBlueprintGraph(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AssetRegistry",
				"BlueprintGraph",
				"Core",
				"CoreUObject",
				"Engine",
				"GraphEditor",
				"HttpBlueprint",
				"HTTP",
				"InputCore",
				"JsonBlueprintUtilities",
				"Kismet",
				"KismetCompiler",
				"KismetWidgets",
				"Slate",
				"SlateCore", 
				"ToolMenus"
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetTools",
					"EditorStyle",
					"Projects",
					"PropertyEditor",
					"UnrealEd",
				}
			);
		}
	}
}