// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpBlueprintGraphLog.h"

DEFINE_LOG_CATEGORY(LogHttpBlueprintEditor);