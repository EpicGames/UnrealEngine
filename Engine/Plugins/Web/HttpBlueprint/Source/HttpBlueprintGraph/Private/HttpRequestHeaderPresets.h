// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"

namespace UE::HttpBlueprint::HeaderPresets
{
	struct FBasePreset
	{
		FBasePreset()
		{
			HeaderPresets =
			{
				{TEXT("Accept-Encoding"), TEXT("identity")},
				{TEXT("User-Agent"), TEXT("X-UnrealEngine-Agent")}
			};
		}

		void SetDefaults(const TMap<FString, FString>& AdditionalHeaders)
		{
			HeaderPresets =
			{
				{TEXT("Accept-Encoding"), TEXT("identity")},
				{TEXT("User-Agent"), TEXT("X-UnrealEngine-Agent")}
			};

			HeaderPresets.Append(AdditionalHeaders);
		}

		TMap<FString, FString> HeaderPresets;
	};

	struct FJsonPreset : FBasePreset
	{
		FJsonPreset()
		{
			SetDefaults({
				{TEXT("Accepts"), TEXT("application/json")},
				{TEXT("Content-Type"), TEXT("application/json")}
			});
		}
	};

	struct FHtmlPreset : FBasePreset
	{
		FHtmlPreset()
		{
			SetDefaults({
				{TEXT("Accepts"), TEXT("text/html")},
				{TEXT("Content-Type"), TEXT("text/html")}
			});
		}
	};

	struct FUrlPreset : FBasePreset
	{
		FUrlPreset()
		{
			SetDefaults({
				{TEXT("Accepts"), TEXT("application/x-www-form-urlencoded")},
				{TEXT("Content-Type"), TEXT("application/x-www-form-urlencoded")}
			});
		}
	};
}
