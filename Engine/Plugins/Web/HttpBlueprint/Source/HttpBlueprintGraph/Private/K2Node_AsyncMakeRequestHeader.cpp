// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_AsyncMakeRequestHeader.h"

#include "HttpRequestProxyObject.h"
#include "EdGraph/EdGraphPin.h"

UK2Node_AsyncMakeRequestHeader::UK2Node_AsyncMakeRequestHeader()
{
	ProxyFactoryFunctionName = GET_FUNCTION_NAME_CHECKED(UHttpRequestProxyObject, CreateHttpRequestProxyObject);
	ProxyFactoryClass = UHttpRequestProxyObject::StaticClass();
	ProxyClass = UHttpRequestProxyObject::StaticClass();
}

UEdGraphPin* UK2Node_AsyncMakeRequestHeader::GetVerbPin() const
{
	UEdGraphPin* Pin = FindPinChecked(TEXT("InVerb"));
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_AsyncMakeRequestHeader::GetBodyPin() const
{
	UEdGraphPin* Pin = FindPinChecked(TEXT("InBody"));
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_AsyncMakeRequestHeader::GetHeaderPin() const
{
	UEdGraphPin* Pin = FindPinChecked(TEXT("InHeader"));
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_AsyncMakeRequestHeader::GetUrlPin() const
{
	UEdGraphPin* Pin = FindPinChecked(TEXT("InUrl"));
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

void UK2Node_AsyncMakeRequestHeader::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// We only want our custom K2Node version to show up, this version should be hidden from the user
}
