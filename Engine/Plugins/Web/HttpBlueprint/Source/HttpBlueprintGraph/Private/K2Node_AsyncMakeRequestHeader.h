// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_BaseAsyncTask.h"

#include "K2Node_AsyncMakeRequestHeader.generated.h"

UCLASS()
class UK2Node_AsyncMakeRequestHeader final : public UK2Node_BaseAsyncTask
{
	GENERATED_BODY()

public:
	UK2Node_AsyncMakeRequestHeader();

	UEdGraphPin* GetVerbPin() const;
    UEdGraphPin* GetBodyPin() const;
    UEdGraphPin* GetHeaderPin() const;
    UEdGraphPin* GetUrlPin() const;

protected:
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
};
