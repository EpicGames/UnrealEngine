// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_HttpRequest.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "HttpBlueprintTypes.h"
#include "HttpHeader.h"
#include "JsonBlueprintFunctionLibrary.h"
#include "K2Node_AsyncMakeRequestHeader.h"
#include "K2Node_CallFunction.h"
#include "K2Node_IfThenElse.h"
#include "KismetCompiler.h"
#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "K2Node_HttpRequest"

namespace HttpRequestLiterals
{
	static const FName SuccessPinName(TEXT("On Success"));
	static const FName ErrorPinName(TEXT("On Error"));
	static const FName UrlPinName(TEXT("Url"));
	static const FName VerbPinName(TEXT("Verb"));
	static const FName HeaderPinName(TEXT("Header"));
	static const FName OutHeaderPinName(TEXT("OutHeader"));
	static const FName BodyPinName(TEXT("Body"));
	static const FName OutputBodyPinName(TEXT("Result Body"));
}

UK2Node_HttpRequest::UK2Node_HttpRequest(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, VerbEnum(FindObject<UEnum>(nullptr, TEXT("/Script/HttpBlueprint.EHttpVerbs")))
{
}

void UK2Node_HttpRequest::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	
	Super::AllocateDefaultPins();
	
	// Execution Pins
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	
	UEdGraphPin* const ThenPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	ThenPin->PinFriendlyName = LOCTEXT("HttpRequest ThenPin", "Request Processing");

	CreatePin(EGPD_Output,
			UEdGraphSchema_K2::PC_Exec,
			HttpRequestLiterals::SuccessPinName)
		->PinFriendlyName = LOCTEXT("HttpRequest SuccessPin", "Request Was Successful");

	CreatePin(EGPD_Output,
			UEdGraphSchema_K2::PC_Exec,
			HttpRequestLiterals::ErrorPinName)
		->PinFriendlyName = LOCTEXT("HttpRequest FailurePin", "Request Was Not Successful");
	
	// Input pins
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_String, HttpRequestLiterals::UrlPinName);
	UEdGraphPin* EnumPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Byte, VerbEnum.Get(), HttpRequestLiterals::VerbPinName);
	Schema->SetPinDefaultValueAtConstruction(EnumPin, VerbEnum->GetNameStringByIndex(0));

	FCreatePinParams PinParams;
	PinParams.bIsConst = true;
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, FHttpHeader::StaticStruct(), HttpRequestLiterals::HeaderPinName, PinParams);

	// Create BodyInputPin. Will only create if the EnumPin has the correct value
	HandleBodyInputPin();

	// Output pins
	CreatePin(EGPD_Output,
			UEdGraphSchema_K2::PC_Struct,
			FHttpHeader::StaticStruct(),
			HttpRequestLiterals::OutHeaderPinName)
		->PinFriendlyName = LOCTEXT("HttpRequest OutHeader", "Header");

	UEdGraphPin* const OutBodyPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, HttpRequestLiterals::OutputBodyPinName);
	SetPinToolTip(OutBodyPin, LOCTEXT("HttpRequest OutBodyPin", "The response from the request"));
}

FText UK2Node_HttpRequest::GetTooltipText() const
{
	return LOCTEXT("HttpRequest_Tooltip", "Processes a Http request ");
}

FText UK2Node_HttpRequest::GetNodeTitle(ENodeTitleType::Type Title) const
{
	const UEdGraphPin* const VerbEnumPin = FindPin(HttpRequestLiterals::VerbPinName);
	return FText::Format(
		LOCTEXT("HttpRequest_Title", "Http {0} Request"),
		FText::FromString(VerbEnumPin ? *VerbEnumPin->DefaultValue : TEXT("")));
}


bool UK2Node_HttpRequest::IsCompatibleWithGraph(const UEdGraph* Graph) const
{
	return Graph->GetSchema()->GetGraphType(Graph) == GT_Ubergraph && Super::IsCompatibleWithGraph(Graph);
}

void UK2Node_HttpRequest::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);
	
	Modify();

	SyncBodyPinType(GetBodyPin());
	SyncBodyPinType(GetOutBodyPin());
}

void UK2Node_HttpRequest::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);

	if (Pin->PinName == HttpRequestLiterals::VerbPinName)
	{
		HandleBodyInputPin();
	}
}

void UK2Node_HttpRequest::PinTypeChanged(UEdGraphPin* Pin)
{
	SyncBodyPinType(GetBodyPin());
	SyncBodyPinType(GetOutBodyPin());
	
	Super::PinTypeChanged(Pin);
}

void UK2Node_HttpRequest::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	UK2Node_AsyncMakeRequestHeader* AsyncRequestNode = CompilerContext.SpawnIntermediateNode<UK2Node_AsyncMakeRequestHeader>(this, SourceGraph);
	AsyncRequestNode->AllocateDefaultPins();

	// Move pins from this node to the proxy object node
	CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *AsyncRequestNode->FindPinChecked(UEdGraphSchema_K2::PN_Then));
	CompilerContext.MovePinLinksToIntermediate(*GetVerbPin(), *AsyncRequestNode->GetVerbPin());
	CompilerContext.MovePinLinksToIntermediate(*GetHeaderPin(), *AsyncRequestNode->GetHeaderPin());
	CompilerContext.MovePinLinksToIntermediate(*GetUrlPin(), *AsyncRequestNode->GetUrlPin());
	
	if (GetBodyPin() && GetBodyPin()->PinType.PinCategory == UEdGraphSchema_K2::PC_String)
	{
		CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *AsyncRequestNode->GetExecPin());
		CompilerContext.MovePinLinksToIntermediate(*GetBodyPin(), *AsyncRequestNode->GetBodyPin());
	}
	else if (GetBodyPin() && GetBodyPin()->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		const FName& FromStructFunctionName = GET_FUNCTION_NAME_CHECKED(UJsonBlueprintFunctionLibrary, StructToJsonString);
		UClass* JsonLibraryClass = UJsonBlueprintFunctionLibrary::StaticClass();

		UK2Node_CallFunction* CallFromStructFunctionNode = SourceGraph->CreateIntermediateNode<UK2Node_CallFunction>();
		CallFromStructFunctionNode->FunctionReference.SetExternalMember(FromStructFunctionName, JsonLibraryClass);
		CallFromStructFunctionNode->AllocateDefaultPins();
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallFromStructFunctionNode, this);

		UEdGraphPin* InStructPin = CallFromStructFunctionNode->FindPinChecked(TEXT("Struct"));
		InStructPin->PinType = GetBodyPin()->PinType;
		
		UEdGraphPin* OutJsonPin = CallFromStructFunctionNode->FindPinChecked(TEXT("OutJsonString"));

		CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *CallFromStructFunctionNode->GetExecPin());
		CompilerContext.MovePinLinksToIntermediate(*GetBodyPin(), *InStructPin);
		OutJsonPin->MakeLinkTo(AsyncRequestNode->GetBodyPin());
		CallFromStructFunctionNode->GetThenPin()->MakeLinkTo(AsyncRequestNode->GetExecPin());
	}
	else if (!GetBodyPin())
	{
		CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *AsyncRequestNode->GetExecPin());
	}
	else if (GetBodyPin())
	{
		CompilerContext.MessageLog.Error(
			*LOCTEXT("Error_InputPinNotStringOrStruct", "Node @@ Attempted to supply a input body pin that was not a string or struct.").
			ToString(), this);
		BreakAllNodeLinks();
	}
	
	UEdGraphPin* FuncExecPin = AsyncRequestNode->GetExecPin();
	FuncExecPin->MakeLinkTo(GetExecPin());
	
	UEdGraphPin* FuncReturnValPin = AsyncRequestNode->FindPinChecked(TEXT("Response"));
	UEdGraphPin* FuncConditionalPin = AsyncRequestNode->FindPinChecked(TEXT("bSuccessful"));
	UEdGraphPin* FuncThenPin = AsyncRequestNode->FindPinChecked(TEXT("OnRequestComplete"));
	UEdGraphPin* FuncOutHeaderPin = AsyncRequestNode->FindPinChecked(TEXT("OutHeader"));
	
	UK2Node_IfThenElse* RequestBranchNode = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
	RequestBranchNode->AllocateDefaultPins();

	CompilerContext.MovePinLinksToIntermediate(*GetErrorPin(), *RequestBranchNode->GetElsePin());
	CompilerContext.MovePinLinksToIntermediate(*GetOutHeaderPin(), *FuncOutHeaderPin);
	
	FuncThenPin->MakeLinkTo(RequestBranchNode->GetExecPin());
	FuncConditionalPin->MakeLinkTo(RequestBranchNode->GetConditionPin());

	/*
	 *	We cache the PinType because in order for us to auto-convert the response data into the user defined struct
	 *	We need to be able to create a JsonObject that can then be deserializated into the user defined struct
	 **/
	const FEdGraphPinType CachedOutputBodyPinType = GetOutBodyPin()->PinType;
	if (CachedOutputBodyPinType.PinCategory != UEdGraphSchema_K2::PC_String)
	{
		GetOutBodyPin()->PinType.ResetToDefaults();
		GetOutBodyPin()->PinType.PinCategory = UEdGraphSchema_K2::PC_String;
	}
	
	// We check the cached pin type because the OutBodyPin should always be a string at this point
	if (CachedOutputBodyPinType.PinCategory == UEdGraphSchema_K2::PC_String)
	{
		CompilerContext.MovePinLinksToIntermediate(*GetSuccessPin(), *RequestBranchNode->GetThenPin());
		CompilerContext.MovePinLinksToIntermediate(*GetErrorPin(), *RequestBranchNode->GetElsePin());
		CompilerContext.MovePinLinksToIntermediate(*GetOutBodyPin(), *FuncReturnValPin);
	}
	else if (CachedOutputBodyPinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		const FName& FromStringFunctionName = GET_FUNCTION_NAME_CHECKED(UJsonBlueprintFunctionLibrary, FromString);
		const FName& GetFieldFunctionName = GET_FUNCTION_NAME_CHECKED(UJsonBlueprintFunctionLibrary, GetField);
		UClass* JsonLibraryClass = UJsonBlueprintFunctionLibrary::StaticClass();

		UK2Node_CallFunction* CallFromStringFunctionNode = SourceGraph->CreateIntermediateNode<UK2Node_CallFunction>();
		CallFromStringFunctionNode->FunctionReference.SetExternalMember(FromStringFunctionName, JsonLibraryClass);
		CallFromStringFunctionNode->AllocateDefaultPins();
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallFromStringFunctionNode, this);		

		UEdGraphPin* JsonStringPin = CallFromStringFunctionNode->FindPinChecked(TEXT("JsonString"));
		UEdGraphPin* JsonObjectPin = CallFromStringFunctionNode->FindPinChecked(TEXT("OutJsonObject"));
		
		UK2Node_CallFunction* CallGetFieldFunctionNode = SourceGraph->CreateIntermediateNode<UK2Node_CallFunction>();
		CallGetFieldFunctionNode->FunctionReference.SetExternalMember(GetFieldFunctionName, JsonLibraryClass);
		CallGetFieldFunctionNode->AllocateDefaultPins();
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallGetFieldFunctionNode, this);

		UEdGraphPin* ObjectWrapperPin = CallGetFieldFunctionNode->FindPinChecked(TEXT("JsonObject"));
		UEdGraphPin* OutValuePin = CallGetFieldFunctionNode->FindPinChecked(TEXT("OutValue"));
		UEdGraphPin* GetFieldConditionalPin = CallGetFieldFunctionNode->GetReturnValuePin();
		
		RequestBranchNode->GetThenPin()->MakeLinkTo(CallFromStringFunctionNode->GetExecPin());
		CompilerContext.MovePinLinksToIntermediate(*GetErrorPin(), *RequestBranchNode->GetElsePin());
		
		FuncReturnValPin->MakeLinkTo(JsonStringPin);

		CallFromStringFunctionNode->GetThenPin()->MakeLinkTo(CallGetFieldFunctionNode->GetExecPin());
		JsonObjectPin->MakeLinkTo(ObjectWrapperPin);

		UK2Node_IfThenElse* JsonBranchNode = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
		JsonBranchNode->AllocateDefaultPins();

		CallGetFieldFunctionNode->GetThenPin()->MakeLinkTo(JsonBranchNode->GetExecPin());
		GetFieldConditionalPin->MakeLinkTo(JsonBranchNode->GetConditionPin());

		OutValuePin->PinType = CachedOutputBodyPinType;
		GetOutBodyPin()->PinType = CachedOutputBodyPinType;

		CompilerContext.MovePinLinksToIntermediate(*GetOutBodyPin(), *OutValuePin);
		CompilerContext.MovePinLinksToIntermediate(*GetSuccessPin(), *JsonBranchNode->GetThenPin());
	}
	else
	{
		CompilerContext.MessageLog.Error(
			*LOCTEXT("Error_DeserializeIntoUnsupportedType", "Node @@ Attempted to deserialize the Result Body into a @@. The supported types are String and Struct").
			ToString(), this, *CachedOutputBodyPinType.PinCategory.ToString());
		BreakAllNodeLinks();
	}

	BreakAllNodeLinks();
}

void UK2Node_HttpRequest::ReconstructNode()
{
	Super::ReconstructNode();

	HandleBodyInputPin();
	SyncBodyPinType(GetBodyPin());
	SyncBodyPinType(GetOutBodyPin());
}

FName UK2Node_HttpRequest::GetCornerIcon() const
{
	return TEXT("Graph.Latent.LatentIcon");
}

void UK2Node_HttpRequest::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	if (const UClass* ActionKey = GetClass();
		ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		checkf(NodeSpawner != nullptr, TEXT("Node spawner failed to create a valid Node"));

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_HttpRequest::GetMenuCategory() const
{
	return LOCTEXT("HttpRequest_Category", "Http");
}

bool UK2Node_HttpRequest::IsConnectionDisallowed(
	const UEdGraphPin* MyPin,
	const UEdGraphPin* OtherPin,
	FString& OutReason) const
{
	if (MyPin == GetOutBodyPin())
	{
		if (OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct || OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_String)
		{
			return false;
		}

		OutReason = LOCTEXT("HttpRequest_InvalidArgumentType_Output", "Output Body may only be a string or struct").ToString();
		return true;
	}
	
	if (MyPin == GetBodyPin())
	{
		const FName& OtherPinCategory = OtherPin->PinType.PinCategory;
		bool bIsValidType = false;

		if (OtherPinCategory == UEdGraphSchema_K2::PC_String || OtherPinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			bIsValidType = true;
		}

		if (!bIsValidType)
		{
			OutReason = LOCTEXT("HttpRequest_InvalidArgumentType_Input", "Input Body may only be a string or struct").ToString();
			return true;
		}
	}
	
	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

void UK2Node_HttpRequest::SetPinToolTip(UEdGraphPin* const& InPin, const FText& ToolTip) const
{
	InPin->PinToolTip = UEdGraphSchema_K2::TypeToText(InPin->PinType).ToString();

	const UEdGraphSchema_K2* const Schema = GetDefault<UEdGraphSchema_K2>();
	InPin->PinToolTip += TEXT("");
	InPin->PinToolTip += Schema->GetPinDisplayName(InPin).ToString();
	InPin->PinToolTip += LINE_TERMINATOR + ToolTip.ToString();
}

void UK2Node_HttpRequest::HandleBodyInputPin()
{
	const UEdGraphPin* const EnumPin = FindPinChecked(HttpRequestLiterals::VerbPinName);

	if (const int32 EnumIndex = GetVerbEnum()->GetIndexByName(*EnumPin->DefaultValue);
		EnumIndex != INDEX_NONE
		&& GetVerbEnum()->GetValueByIndex(EnumIndex) <= StaticCast<int32>(EHttpVerbs::Patch))
	{
		if (FindPin(HttpRequestLiterals::BodyPinName) == nullptr)
		{
			CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, HttpRequestLiterals::BodyPinName);
		}
		
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
		return;
	}

	if (UEdGraphPin* const BodyPin = FindPin(HttpRequestLiterals::BodyPinName);
		BodyPin != nullptr)
	{
		RemovePin(BodyPin);
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

UEnum* UK2Node_HttpRequest::GetVerbEnum() const
{
	return VerbEnum ? VerbEnum.Get() : FindObjectChecked<UEnum>(nullptr, TEXT("/Script/HttpBlueprint.EHttpVerbs"));
}

void UK2Node_HttpRequest::SyncBodyPinType(UEdGraphPin* const Pin) const
{
	// Early out for cases where the input body pin is nullptr
	if (!Pin)
	{
		return;
	}

	bool bBodyTypeChanged = false;

	if (Pin->LinkedTo.Num() == 0)
	{
		if (static const FEdGraphPinType WildcardType{
			UEdGraphSchema_K2::PC_Wildcard,
			NAME_None,
			nullptr,
			EPinContainerType::None,
			false,
			{}};
		Pin->PinType != WildcardType)
		{
			Pin->PinType = WildcardType;
			bBodyTypeChanged = true;
		}
	}
	else
	{
		if (const FEdGraphPinType SourcePinType = Pin->LinkedTo[0]->PinType;
			Pin->PinType != SourcePinType)
		{
			Pin->PinType = SourcePinType;
			bBodyTypeChanged = true;
		}
	}

	if (bBodyTypeChanged)
	{
		GetGraph()->NotifyNodeChanged(this);
		if (UBlueprint* const BP = GetBlueprint(); !BP->bBeingCompiled)
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(BP);
			BP->BroadcastChanged();
		}
	}
}

UEdGraphPin* UK2Node_HttpRequest::GetVerbPin() const
{
	UEdGraphPin* Pin = FindPinChecked(HttpRequestLiterals::VerbPinName);
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_HttpRequest::GetOutBodyPin() const
{
	UEdGraphPin* Pin = FindPinChecked(HttpRequestLiterals::OutputBodyPinName);
	check(Pin->Direction == EGPD_Output);
	return Pin;
}

UEdGraphPin* UK2Node_HttpRequest::GetBodyPin() const
{
	// The input body is dynamic. So there are cases where this will return nullptr
	UEdGraphPin* Pin = FindPin(HttpRequestLiterals::BodyPinName);
	check(Pin ? Pin->Direction == EGPD_Input : true);
	return Pin;
}

UEdGraphPin* UK2Node_HttpRequest::GetHeaderPin() const
{
	UEdGraphPin* Pin = FindPinChecked(HttpRequestLiterals::HeaderPinName);
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_HttpRequest::GetOutHeaderPin() const
{
	UEdGraphPin* Pin = FindPinChecked(HttpRequestLiterals::OutHeaderPinName);
	check(Pin->Direction == EGPD_Output);
	return Pin;
}

UEdGraphPin* UK2Node_HttpRequest::GetUrlPin() const
{
	UEdGraphPin* Pin = FindPinChecked(HttpRequestLiterals::UrlPinName);
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_HttpRequest::GetSuccessPin() const
{
	UEdGraphPin* Pin = FindPinChecked(HttpRequestLiterals::SuccessPinName);
	check(Pin->Direction == EGPD_Output);
	return Pin;
}

UEdGraphPin* UK2Node_HttpRequest::GetErrorPin() const
{
	UEdGraphPin* Pin = FindPinChecked(HttpRequestLiterals::ErrorPinName);
	check(Pin->Direction == EGPD_Output);
	return Pin;
}

#undef LOCTEXT_NAMESPACE
