// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"

#include "K2Node_HttpRequest.generated.h"

namespace ENodeTitleType { enum Type : int; }

UCLASS(MinimalAPI)
class UK2Node_HttpRequest : public UK2Node
{
	GENERATED_BODY()

public:
	explicit UK2Node_HttpRequest(const FObjectInitializer& ObjectInitializer);
	
	/** Begin UEdGraphNode Interface */
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* Graph) const override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual void PinTypeChanged(UEdGraphPin* Pin) override;
	/** End UEdGraphNode Interface */

	/** Begin UK2Node Interface */
	virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void ReconstructNode() override;
	virtual FName GetCornerIcon() const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual FText GetMenuCategory() const override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	/** Begin UK2Node Interface */

protected:
	void SetPinToolTip(UEdGraphPin* const& InPin, const FText& ToolTip) const;
	
	void HandleBodyInputPin();
	UEnum* GetVerbEnum() const;

	void SyncBodyPinType(UEdGraphPin* const Pin) const;

	UEdGraphPin* GetVerbPin() const;
	UEdGraphPin* GetOutBodyPin() const;
	UEdGraphPin* GetBodyPin() const;
	UEdGraphPin* GetHeaderPin() const;
	UEdGraphPin* GetOutHeaderPin() const;
	UEdGraphPin* GetUrlPin() const;
	UEdGraphPin* GetSuccessPin() const;
	UEdGraphPin* GetErrorPin() const;
	
private:
	UPROPERTY()
	TObjectPtr<UEnum> VerbEnum;
};
