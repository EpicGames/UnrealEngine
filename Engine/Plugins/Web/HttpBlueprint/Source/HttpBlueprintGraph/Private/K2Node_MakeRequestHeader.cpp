// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_MakeRequestHeader.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "HttpBlueprintFunctionLibrary.h"
#include "HttpBlueprintGraphLog.h"
#include "HttpBlueprintTypes.h"
#include "HttpHeader.h"
#include "K2Node_CallFunction.h"
#include "K2Node_MakeMap.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"
#include "ScopedTransaction.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "K2Node_MakeRequestHeader"

namespace UE::HttpBlueprint::Private
{
	namespace MakeRequestHeaderGlobals
	{
		static const FName OutputPinName(TEXT("Header"));
		static const FName PresetPinName(TEXT("Preset"));

		static const FName InputKeyPrefix(TEXT("Header"));
		static const FName InputValuePrefix(TEXT("Value"));
		
		static TArray<TPair<FString, FString>> Presets;
	}

	namespace Pin
	{
		[[nodiscard]] static FString MakePinName(int32 PinIndex)
		{
			const int32 PairIndex = PinIndex / 2;
			if (PinIndex % 2 == 0)
			{
				return *FString::Printf(TEXT("%s %d"), *UE::HttpBlueprint::Private::MakeRequestHeaderGlobals::InputKeyPrefix.ToString(), PairIndex);
			}

			return *FString::Printf(TEXT("%s %d"), *UE::HttpBlueprint::Private::MakeRequestHeaderGlobals::InputValuePrefix.ToString(), PairIndex);
		}
	
		[[nodiscard]] static bool IsValidInputPin(const UEdGraphPin* InputPin)
		{
			return InputPin
				&& InputPin->Direction == EGPD_Input
				&& InputPin->PinName != UE::HttpBlueprint::Private::MakeRequestHeaderGlobals::PresetPinName
				&& InputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_String;
		}

		/** Returns true if the Pin has any connection or an inline, non-default value. */
		static bool PinHasConnectionOrValue(const UEdGraphPin* Pin)
		{
			return Pin->HasAnyConnections() || !Pin->DefaultValue.IsEmpty();
		}

		/** Returns true if one or more pins match the condition. */
		static TArray<UEdGraphPin*> CopyKeyPinsIf(
			TArrayView<UEdGraphPin*> From,
			TUniqueFunction<bool(const UEdGraphPin*)>&& PredicateFunc)
		{
			TArray<UEdGraphPin*> To;
			To.Reserve(From.Num());

			for (int32 Index{0}; Index < From.Num(); Index += 2)
			{
				UEdGraphPin* Pin = From[Index];
				if (UE::HttpBlueprint::Private::Pin::IsValidInputPin(Pin)
					&& PredicateFunc(Pin))
				{
					To.Add(Pin);					
				}
			}

			return To;
		}

		/** A Key Pin is considered valid if it has an input or non-default value. */
		static TArray<UEdGraphPin*> GetValidKeyPins(TArrayView<UEdGraphPin*> Pins)
		{
			TArray<UEdGraphPin*> EmptyKeyPins = CopyKeyPinsIf(Pins,
				[](const UEdGraphPin* KeyPin)
				{
					return PinHasConnectionOrValue(KeyPin);
				});

			return EmptyKeyPins;
		}
	}

	// This exists so we can just store an array of the presets instead of a map.
	// This allows us to avoid looping and comparing Enum values.
	// Instead we can just use random access since the index is known
	namespace FPresetPair
	{
		namespace Private
		{
			static FString GetPresetValueFromEnum(const ERequestPresets& InPresetEnum)
			{
				switch (InPresetEnum)
				{
					case ERequestPresets::Json:		return TEXT("application/json");
					case ERequestPresets::Http:		return TEXT("text/html");
					case ERequestPresets::Url:		return TEXT("application/x-www-form-urlencoded");
					case ERequestPresets::Custom:	return TEXT("");
					default:						return TEXT("");
				}
			}
		}

		inline namespace Public
		{
			static FString PresetArrayToString()
			{
				FString OutString = TEXT("Preset String");
				for (int32 Index{0}; Index < MakeRequestHeaderGlobals::Presets.Num(); Index++)
				{
					OutString = FString::Printf(TEXT("%s%s: %s"), LINE_TERMINATOR,
					                            *MakeRequestHeaderGlobals::Presets[Index].Key,
					                            *MakeRequestHeaderGlobals::Presets[Index].Value);
				}

				return OutString;
			}
			
			static FString GetHeaderValueForIndex(int32 InIndex, int32 InEnumIndex)
			{
				if (!MakeRequestHeaderGlobals::Presets.IsValidIndex(InIndex))
				{
					FFrame::KismetExecutionMessage(*FString::Printf(
						TEXT("Preset array was not properly initialized. Can not make a request header. Index: %i -- Presets: %s"), InIndex, *PresetArrayToString()),
						ELogVerbosity::Error);
					return {};
				}

				const FString& HeaderValue = MakeRequestHeaderGlobals::Presets[InIndex].Value;
				return HeaderValue.IsEmpty() ? Private::GetPresetValueFromEnum(
						       StaticCast<ERequestPresets>(InEnumIndex)) : HeaderValue;
			}

			static FString GetHeaderKeyForIndex(int32 InIndex)
			{
				if (!MakeRequestHeaderGlobals::Presets.IsValidIndex(InIndex))
				{
					FFrame::KismetExecutionMessage(*FString::Printf(
					TEXT("Preset array was not properly initialized. Can not make a request header. Index: %i -- Presets: %s"), InIndex, *PresetArrayToString()),
						ELogVerbosity::Error);
					return {};
				}
				
				return MakeRequestHeaderGlobals::Presets[InIndex].Key;
			}

			static int32 NumPresetHeaders()
			{
				return MakeRequestHeaderGlobals::Presets.Num();
			}
		}
	}
}

UK2Node_MakeRequestHeader::UK2Node_MakeRequestHeader(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NumInputs = 1;
	PresetEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/HttpBlueprint.ERequestPresets"));

	UE::HttpBlueprint::Private::MakeRequestHeaderGlobals::Presets =
	{
		{ TTuple<FString, FString>("Content-Type", "") },
		{ TTuple<FString, FString>("Accepts", "") },
		{ TTuple<FString, FString>("User-Agent", "X-UnrealEngine-Agent") },
		{ TTuple<FString, FString>("Accept-Encoding", "identity") },
	};
}

void UK2Node_MakeRequestHeader::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	// @note: Important to do this here for default pin allocation to work (which is called below)
	NumInputs = FMath::Max(1, (OldPins.Num() - 2) / 2);
	
	Super::ReallocatePinsDuringReconstruction(OldPins);
}

void UK2Node_MakeRequestHeader::AllocateDefaultPins()
{
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, FHttpHeader::StaticStruct(), UE::HttpBlueprint::Private::MakeRequestHeaderGlobals::OutputPinName);

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	UEdGraphPin* PresetPin = CreatePin(
		EGPD_Input,
		UEdGraphSchema_K2::PC_Byte,
		PresetEnum.Get(),
		UE::HttpBlueprint::Private::MakeRequestHeaderGlobals::PresetPinName);
	Schema->SetPinDefaultValueAtConstruction(PresetPin, PresetEnum->GetNameStringByIndex(PresetEnumIndex));

	// Create the input pins to create the container from
	for (int32 InputIndex = 0; InputIndex < NumInputs; ++InputIndex)
	{
		FString KeyPinName = UE::HttpBlueprint::Private::Pin::MakePinName(InputIndex * 2);
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_String,*KeyPinName);
		
		FString ValuePinName = UE::HttpBlueprint::Private::Pin::MakePinName(InputIndex * 2 + 1);
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_String, *ValuePinName);
	}

	SyncPinNames();

	ConstructDefaultPinsForPreset();
}

FText UK2Node_MakeRequestHeader::GetNodeTitle(ENodeTitleType::Type Title) const
{
	return LOCTEXT("MakeRequestHeader_Title", "Make Request Header");
}

FText UK2Node_MakeRequestHeader::GetTooltipText() const
{
	return LOCTEXT("MakeRequestHeader_Tooltip", "Creates a Header object that can be used to send Http Requests");
}

void UK2Node_MakeRequestHeader::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);

	if (Pin->Direction == EGPD_Input
		&& Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Byte)
	{
		ConstructDefaultPinsForPreset();
	}
}

void UK2Node_MakeRequestHeader::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	if (!Context->bIsDebugging)
	{
		FToolMenuSection& Section = Menu->AddSection("K2Node_MakeRequestHeader", LOCTEXT("MakeRequestHeader_ContextMenu", "Make Request Header"));
		
		if (Context->Pin)
		{
			if (Context->Pin->Direction == EGPD_Input && !Context->Pin->ParentPin)
			{
				Section.AddMenuEntry("RemovePin", LOCTEXT("MakeRequestHeader_RemovePin", "Remove header pin"),
				                     LOCTEXT("MakeRequestHeader_RemovePin_Tooltip", "Remove this header element pin"),
				                     FSlateIcon(),
				                     FUIAction(FExecuteAction::CreateUObject(
					                     const_cast<UK2Node_MakeRequestHeader*>(this),
					                     &UK2Node_MakeRequestHeader::RemoveInputPin,
					                     const_cast<UEdGraphPin*>(Context->Pin))));
			}
		}
		else
		{
			Section.AddMenuEntry("AddPin", LOCTEXT("MakeRequestHeader_AddPin", "Add header pin"),
			                     LOCTEXT("MakeRequestHeader_AddPin_Tooltip", "Add another header pin"),
			                     FSlateIcon(),
			                     FUIAction(FExecuteAction::CreateUObject(
				                     const_cast<UK2Node_MakeRequestHeader*>(this),
				                     &UK2Node_MakeRequestHeader::AddInputPin)));
		}
	}
}

void UK2Node_MakeRequestHeader::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
	
	const FName& FunctionName = GET_FUNCTION_NAME_CHECKED(UHttpBlueprintFunctionLibrary, MakeRequestHeader);
	UClass* FunctionClass = UHttpBlueprintFunctionLibrary::StaticClass();

	UK2Node_CallFunction* CallFunctionNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallFunctionNode->FunctionReference.SetExternalMember(FunctionName, FunctionClass);
	CallFunctionNode->AllocateDefaultPins();
	CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallFunctionNode, this);

	UK2Node_MakeMap* MakeMapNode = CompilerContext.SpawnIntermediateNode<UK2Node_MakeMap>(this, SourceGraph);
	MakeMapNode->AllocateDefaultPins();
	CompilerContext.MessageLog.NotifyIntermediateObjectCreation(MakeMapNode, this);

	UEdGraphPin* MapOutPin = MakeMapNode->GetOutputPin();

	UEdGraphPin* CallFunctionInputPin = CallFunctionNode->FindPinChecked(TEXT("Headers"));
	MapOutPin->MakeLinkTo(CallFunctionInputPin);
	MakeMapNode->PinConnectionListChanged(MapOutPin);

	const bool bIsUserSpecifiedHeader = PresetEnumIndex == StaticCast<int32>(ERequestPresets::Custom);
	if (bIsUserSpecifiedHeader)
	{
		TArray<UEdGraphPin*> ValidKeyPins = UE::HttpBlueprint::Private::Pin::GetValidKeyPins(Pins);

		// Add input pins to the MakeMap node
		// UK2Node_MakeMap::AddInputPin adds two pins per call which is why we only need to call it for Pins.Num / 2
		for (int32 Index{0}; Index < ValidKeyPins.Num(); ++Index)
		{
			MakeMapNode->AddInputPin();
		}

		for (int32 Index{0}; Index < ValidKeyPins.Num(); ++Index)
		{
			UEdGraphPin* KeyPin = ValidKeyPins[Index];
			const int32 KeyPinIndex = GetPinIndex(KeyPin);

			const int32 MapKeyPinIndex = Index * 2;

			UEdGraphPin* MapKeyPin = MakeMapNode->FindPinChecked(MakeMapNode->GetPinName(MapKeyPinIndex));
			MapKeyPin->PinType = KeyPin->PinType;
			CompilerContext.MovePinLinksToIntermediate(*KeyPin, *MapKeyPin);

			UEdGraphPin* ValuePin = GetPinAt(KeyPinIndex + 1);
			
			UEdGraphPin* MapValuePin = MakeMapNode->FindPinChecked(MakeMapNode->GetPinName(MapKeyPinIndex + 1));
			MapValuePin->PinType = ValuePin->PinType;
			CompilerContext.MovePinLinksToIntermediate(*ValuePin, *MapValuePin);
		}
	}
	else
	{
		// MakeMapNode spawns one set of pins by default. So we only need to Spawn NumPins - 1
		for (int32 Index{0}; Index < UE::HttpBlueprint::Private::FPresetPair::NumPresetHeaders() - 1; Index++)
		{
			MakeMapNode->AddInputPin();
		}

		TArray<UEdGraphPin*> KeyPins, ValuePins;
		MakeMapNode->GetKeyAndValuePins(KeyPins, ValuePins);
		for (int32 Index{0}; Index < KeyPins.Num(); Index++)
		{
			UEdGraphPin* KeyPin = KeyPins[Index];
			UEdGraphPin* ValuePin = ValuePins[Index];

			Schema->TrySetDefaultValue(*KeyPin, UE::HttpBlueprint::Private::FPresetPair::GetHeaderKeyForIndex(Index));
			Schema->TrySetDefaultValue(*ValuePin, UE::HttpBlueprint::Private::FPresetPair::GetHeaderValueForIndex(Index, PresetEnumIndex));
		}
	}

	if (UEdGraphPin* OutputPin = FindPin(UE::HttpBlueprint::Private::MakeRequestHeaderGlobals::OutputPinName))
	{
		if (UEdGraphPin* HeaderOutputPin = CallFunctionNode->FindPin(TEXT("OutHeader")))
		{
			CompilerContext.MovePinLinksToIntermediate(*OutputPin, *HeaderOutputPin);			
		}
	}

	BreakAllNodeLinks();
}

void UK2Node_MakeRequestHeader::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	if (const UClass* ActionKey = GetClass();
		ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		checkf(NodeSpawner != nullptr, TEXT("Node spawner failed to create a valid Node"));

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_MakeRequestHeader::GetMenuCategory() const
{
	return LOCTEXT("MakeRequestHeader_Category", "Http");
}

bool UK2Node_MakeRequestHeader::NodeCausesStructuralBlueprintChange() const
{
	return true;
}

bool UK2Node_MakeRequestHeader::IsNodePure() const
{
	return true;
}

UEdGraphPin* UK2Node_MakeRequestHeader::GetOutputPin() const
{
	return FindPinChecked(UE::HttpBlueprint::Private::MakeRequestHeaderGlobals::OutputPinName);
}

void UK2Node_MakeRequestHeader::ConstructDefaultPinsForPreset()
{
	const UEdGraphPin* PresetPin = FindPinChecked(UE::HttpBlueprint::Private::MakeRequestHeaderGlobals::PresetPinName);
	if (const int32 EnumIndex = PresetEnum->GetIndexByName(*PresetPin->DefaultValue);
		EnumIndex != INDEX_NONE)
	{
		PresetEnumIndex = EnumIndex;
		if (PresetEnumIndex == StaticCast<int32>(ERequestPresets::Custom))
		{
			if (OptionalPins.Num() > 0)
			{
				const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
				for (const FOptionalPin& OptionalPin : OptionalPins)
				{
					UEdGraphPin* ExistingOrNewPin = FindPin(OptionalPin.PinName);
					if (!ExistingOrNewPin)
					{
						ExistingOrNewPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_String, OptionalPin.PinName);
					}
					
					if (UEdGraphPin* LinkedToPin = OptionalPin.LinkedTo.Get())
					{
						ExistingOrNewPin->MakeLinkTo(LinkedToPin);						
					}
					
					Schema->SetPinDefaultValueAtConstruction(ExistingOrNewPin, OptionalPin.PinDefaultValue);
				}
				NumInputs = (Pins.Num() - 2) / 2;
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
			}
		}
		else
		{
			// Remove all pins except for the preset and output pin
			// Store all custom pins for reconstruction later
			OptionalPins.Empty(Pins.Num() - 2);
			for (int32 Index{2}; Index < Pins.Num(); ++Index)
			{
				UEdGraphPin* PinToRemove = Pins.IsValidIndex(Index) ? Pins[Index] : nullptr;
				if (PinToRemove == nullptr)
				{
					UE_LOGF(LogHttpBlueprintEditor, Warning, "Pin was invalid");
					break;
				}
				
				FScopedTransaction Transaction(LOCTEXT("RemovePinTx", "RemovePin"));
				Modify();

				FOptionalPin NewOptionalPin = MakeOptionalPin(PinToRemove);
				OptionalPins.Add(NewOptionalPin);
				
				TFunction<void(UEdGraphPin*)> RemovePinLambda = [this, &RemovePinLambda](UEdGraphPin* PinToRemove)->void
				{
					check(PinToRemove);
					for (int32 SubPinIndex = PinToRemove->SubPins.Num() - 1; SubPinIndex >= 0; --SubPinIndex)
					{
						RemovePinLambda(PinToRemove->SubPins[SubPinIndex]);
					}

					int32 PinRemovalIndex = INDEX_NONE;
					if (Pins.Find(PinToRemove, PinRemovalIndex))
					{
						Pins.RemoveAt(PinRemovalIndex);
						PinToRemove->MarkAsGarbage();
					}
				};
				
				RemovePinLambda(PinToRemove);
				PinConnectionListChanged(PinToRemove);
				--Index;
			}

			NumInputs = 0;
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
		}
	}
}

bool UK2Node_MakeRequestHeader::CanAddPin() const
{
	return StaticCast<ERequestPresets>(PresetEnumIndex) == ERequestPresets::Custom;
}

bool UK2Node_MakeRequestHeader::CanRemovePin(const UEdGraphPin* Pin) const
{
	return StaticCast<ERequestPresets>(PresetEnumIndex) == ERequestPresets::Custom
		&& Pin->Direction == EGPD_Input
		&& Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_String;
}

void UK2Node_MakeRequestHeader::AddInputPin()
{
	FScopedTransaction Transaction(LOCTEXT("AddPinTx", "Add Pin"));
	Modify();

	++NumInputs;
	const FString KeyPinName = UE::HttpBlueprint::Private::Pin::MakePinName((NumInputs - 1) * 2);
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_String,
	          *FString::Printf(TEXT("%s"), *KeyPinName));

	const FString ValuePinName = UE::HttpBlueprint::Private::Pin::MakePinName((NumInputs - 1) * 2 + 1);
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_String,
	          *FString::Printf(TEXT("%s"), *ValuePinName));
	
	if (!GetBlueprint()->bBeingCompiled)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
		GetGraph()->NotifyNodeChanged(this);
	}
}

void UK2Node_MakeRequestHeader::RemoveInputPin(UEdGraphPin* Pin)
{
	FScopedTransaction Transaction(LOCTEXT("RemovePinTx", "RemovePin"));
	Modify();

	ForEachInputPin(Pins, [this](UEdGraphPin* Pin, const int32& PinIndex)
	{
		Pins.RemoveAt(PinIndex);
		Pin->MarkAsGarbage();
		--NumInputs;
	});

	SyncPinNames();
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
}

void UK2Node_MakeRequestHeader::ForEachInputPin(
	TArrayView<UEdGraphPin*> InPins,
	TUniqueFunction<void(UEdGraphPin*, int32)>&& PinFunc)
{
	TFunction<bool(UEdGraphPin*)> ForEachPinLambda = [this, &ForEachPinLambda, &PinFunc](UEdGraphPin* PinToAffect)
	{
		check(PinToAffect);
		
		for (int32 SubPinIndex = PinToAffect->SubPins.Num() - 1; SubPinIndex >= 0; --SubPinIndex)
		{
			ForEachPinLambda(PinToAffect->SubPins[SubPinIndex]);
		}

		int32 PinIndex = INDEX_NONE;
		if (Pins.Find(PinToAffect, PinIndex))
		{
			PinFunc(PinToAffect, PinIndex);
			return true;
		}

		return false;
	};

	for (UEdGraphPin*& Pin : InPins)
	{
		if (!Pin)
		{
			continue;
		}

		if (Pin->Direction != EEdGraphPinDirection::EGPD_Input)
		{
			continue;
		}

		if (!Pins.Contains(Pin))
		{
			UE_LOGF(LogHttpBlueprintEditor, Warning, "Pin %ls wasn't valid.", *Pin->GetName());
			continue;
		}

		TArray<UEdGraphPin*> KeyPins;
		TArray<UEdGraphPin*> ValuePins;
		GetKeyAndValuePins(Pins, KeyPins, ValuePins);

		int32 KVPPinIndex = INDEX_NONE;
		
		// If it's a value pin, then remove the key pin which necessarily exists
		if (ValuePins.Find(Pin, KVPPinIndex))
		{
			ForEachPinLambda(KeyPins[KVPPinIndex]);
		}
		// Otherwise the inverse is true
		else
		{
			verify(KeyPins.Find(Pin, KVPPinIndex));
			ForEachPinLambda(ValuePins[KVPPinIndex]);
		}

		// What's left - either the key or value, depending on the above
		if (ForEachPinLambda(Pin))
		{
			PinConnectionListChanged(Pin);
		}
		else
		{
			UE_LOGF(LogHttpBlueprintEditor, Warning, "Pin %ls wasn't affected.", *Pin->GetName());
		}
	}
}

FOptionalPin UK2Node_MakeRequestHeader::MakeOptionalPin(UEdGraphPin* InPinToCopy)
{
	FOptionalPin NewOptionalPin;
	if (!InPinToCopy->LinkedTo.IsEmpty())
	{
		NewOptionalPin.LinkedTo = InPinToCopy->LinkedTo[0];
	}
				
	NewOptionalPin.PinName = InPinToCopy->PinName;
	NewOptionalPin.PinDefaultValue = InPinToCopy->DefaultValue;
	return NewOptionalPin;
}

void UK2Node_MakeRequestHeader::SyncPinNames()
{
	int32 CurrentNumParentPins = 0;
	for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
	{
		UEdGraphPin*& CurrentPin = Pins[PinIndex];
		if (CurrentPin->ParentPin == nullptr
			&& UE::HttpBlueprint::Private::Pin::IsValidInputPin(Pins[PinIndex]))
		{
			const FName OldName = CurrentPin->PinName;
			const FString ElementName = UE::HttpBlueprint::Private::Pin::MakePinName(CurrentNumParentPins++);

			CurrentPin->Modify();
			CurrentPin->PinName = FName(ElementName);

			if (CurrentPin->SubPins.Num() > 0)
			{
				const FString OldNameStr = OldName.ToString();
				const FString ElementNameStr = ElementName;
				FString OldFriendlyName = OldNameStr;
				FString ElementFriendlyName = ElementNameStr;

				OldFriendlyName.InsertAt(1, " ");
				ElementFriendlyName.InsertAt(1, " ");

				for (UEdGraphPin* SubPin : CurrentPin->SubPins)
				{
					FString SubPinFriendlyName = SubPin->PinFriendlyName.ToString();
					SubPinFriendlyName.ReplaceInline(*OldFriendlyName, *ElementFriendlyName);

					SubPin->Modify();
					SubPin->PinName = *SubPin->PinName.ToString().Replace(*OldNameStr, *ElementNameStr);
					SubPin->PinFriendlyName = FText::FromString(SubPinFriendlyName);
				}
			}
		}
	}
}

void UK2Node_MakeRequestHeader::GetKeyAndValuePins(TArrayView<UEdGraphPin*> InPins, TArray<UEdGraphPin*>& KeyPins, TArray<UEdGraphPin*>& ValuePins) const
{
	KeyPins.Reserve(InPins.Num());
	ValuePins.Reserve(InPins.Num());

	// Store the last found key pin, such that PinFunc takes both the key and value pins
	UEdGraphPin* LastPin = nullptr;
		
	// skip the first two (input, output) pins
	for (int32 PinIndex = 2; PinIndex < InPins.Num(); ++PinIndex)
	{
		UEdGraphPin* CurrentPin = InPins[PinIndex];
		if (CurrentPin->Direction == EGPD_Input && CurrentPin->ParentPin == nullptr)
		{
			// Key/Value pins alternate so if the PinIndex is even, then this is a key
			if (PinIndex % 2 == 0) 
			{
				LastPin = CurrentPin;
			}
			else
			{
				KeyPins.Add(LastPin);
				ValuePins.Add(CurrentPin);
			}
		}
	}

	check(KeyPins.Num() == ValuePins.Num());
}

#undef LOCTEXT_NAMESPACE
