// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"
#include "K2Node_AddPinInterface.h"

#include "K2Node_MakeRequestHeader.generated.h"

namespace ENodeTitleType { enum Type : int; }
class FCompilerResultsLog;
struct FEdGraphPinReference;
class UGraphNodeContextMenuContext;
class UToolMenu;

/** Stores pin connections to restore after a node rebuild. */
USTRUCT()
struct FOptionalPin
{
	GENERATED_BODY()

	/** Pin Name. */
	UPROPERTY()
	FName PinName;

	/** Default value, stored as a string. */
	UPROPERTY()
	FString PinDefaultValue;

	/** Optional, use if something was previously linked to the pin. */
	UPROPERTY()
	FEdGraphPinReference LinkedTo;
};

/** Node to create an Http header, with presets. */
UCLASS(MinimalAPI)
class UK2Node_MakeRequestHeader
	: public UK2Node
	, public IK2Node_AddPinInterface
{
	GENERATED_BODY()

public:
	UK2Node_MakeRequestHeader(const FObjectInitializer& ObjectInitializer);

	/** Begin UEdGraphNode Interface */
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	/** End UEdGraphNode Interface */

	/** Begin UK2Node Interface */
	virtual void GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const override;
	virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual bool NodeCausesStructuralBlueprintChange() const override;
	virtual bool IsNodePure() const override;
	/** Begin UK2Node Interface */

	/** Begin IK2Node_AddPinInterface */
	virtual bool CanAddPin() const override;
	virtual bool CanRemovePin(const UEdGraphPin* Pin) const override;
	virtual void AddInputPin() override;
	virtual void RemoveInputPin(UEdGraphPin* Pin) override;
	/** End IK2Node_AddPinInterface */

	/** Returns the singular output pin containing the created http header. */
	UEdGraphPin* GetOutputPin() const;

protected:
	/** Creates the default pin layout for the currently selected preset. */
	void ConstructDefaultPinsForPreset();

	/** Updates the pin names according to their type and input index. */
	void SyncPinNames();

	/** Separates and returns the Key and Value pins corresponding to each input Key/Value pair. */
	virtual void GetKeyAndValuePins(TArrayView<UEdGraphPin*> InPins, TArray<UEdGraphPin*>& KeyPins, TArray<UEdGraphPin*>& ValuePins) const;

	/** Iterates over the provided Pins. */
	void ForEachInputPin(TArrayView<UEdGraphPin*> InPins, TUniqueFunction<void(UEdGraphPin*,int32)>&& PinFunc);

	/** Creates a copy of the input Pin as an Optional Pin. */
	static FOptionalPin MakeOptionalPin(UEdGraphPin* InPinToCopy);

private:
	/** The UEnum type containing preset names. */
	TObjectPtr<UEnum> PresetEnum;

	/** The currently selected preset index. */
	UPROPERTY()
	int32 PresetEnumIndex;

	/** Number of key/value inputs. */
	UPROPERTY()
	int32 NumInputs;

	/** Used to store hidden or pins to be later restored. */
	UPROPERTY()
	TArray<FOptionalPin> OptionalPins;
};
